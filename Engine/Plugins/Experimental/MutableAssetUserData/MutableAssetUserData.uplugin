{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mutable Asset User Data",
	"Description": "Adds Asset User Data type and operations.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": 
	[
		{
			"Name": "MutableAssetUserData",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": 
	[
		{
			"Name": "Mutable",
			"Enabled": true
		}
	]
}