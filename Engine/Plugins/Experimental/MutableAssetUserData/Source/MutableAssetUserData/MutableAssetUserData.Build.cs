// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MutableAssetUserData : ModuleRules
	{
        public MutableAssetUserData(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(ModuleDirectory + "/Private");
			PublicIncludePaths.Add(ModuleDirectory + "/Public");
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"Core",
					"CoreUObject",
					"MutableRuntime"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CustomizableObject"
				});
		}
	}
}
