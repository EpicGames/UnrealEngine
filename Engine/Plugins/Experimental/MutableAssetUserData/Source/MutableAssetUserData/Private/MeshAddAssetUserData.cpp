// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshAddAssetUserData.h"

#include "MutableAssetUserData.h"
#include "MuR/Mesh.h"
#include "MuR/External/MeshAdapter.h"
#include "Engine/AssetUserData.h"

#define LOCTEXT_NAMESPACE "MutableAssetUserData"


const FText FMeshAddAssetUserData::TextInputMesh = LOCTEXT("Mesh", "Mesh");

const FText FMeshAddAssetUserData::TextInputAssetUserData = LOCTEXT("AssetUserData", "Asset User Data");


TArray<TPair<FText, const UScriptStruct*>> FMeshAddAssetUserData::GetInputs() const
{
	TArray<TPair<FText, const UScriptStruct*>> Inputs;

	Inputs.Emplace(TextInputMesh, UE::Mutable::FMeshAdapter::StaticStruct());
	Inputs.Emplace(TextInputAssetUserData, FMutableAssetUserData::StaticStruct());
	
	return Inputs;
}


TPair<FText, const UScriptStruct*> FMeshAddAssetUserData::GetOutput() const
{
	return MakeTuple(LOCTEXT("Mesh", "Mesh"), UE::Mutable::FMeshAdapter::StaticStruct());
}


void FMeshAddAssetUserData::Evaluate(UE::Mutable::FContext& Context) const
{
	UE::Mutable::FValue InputMesh = UE::Mutable::CopyOrMove(Context.GetInput(TextInputMesh));
	UE::Mutable::FMeshAdapter& MeshAdapter = InputMesh.Get<UE::Mutable::FMeshAdapter>();

	UE::Mutable::FValueConst InputAssetUserData = Context.GetInput(TextInputAssetUserData);
	const FMutableAssetUserData& AssetUserData = InputAssetUserData.Get<FMutableAssetUserData>();
	
	MeshAdapter.GetPrivate()->AssetUserData.Add(UE::Mutable::Private::TPassthroughObjectPtr<UAssetUserData>(AssetUserData.AssetUserData));
	
	// Output the modified Mesh Adapter.
	Context.SetOutput(MoveTemp(InputMesh));
}


#undef LOCTEXT_NAMESPACE