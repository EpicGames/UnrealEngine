// Copyright Epic Games, Inc. All Rights Reserved.

#include "MutableAssetUserDataModule.h"

#include "MeshAddAssetUserData.h"
#include "MuCO/ICustomizableObjectModule.h"


IMPLEMENT_MODULE(FMutableAssetUserDataModule, MutableAssetUserData);


void FMutableAssetUserDataModule::StartupModule()
{
	ICustomizableObjectModule& Module = ICustomizableObjectModule::Get();

	Module.RegisterExternalOperation(FMeshAddAssetUserData::StaticStruct());
}


void FMutableAssetUserDataModule::ShutdownModule()
{
	ICustomizableObjectModule& Module = ICustomizableObjectModule::Get();

	Module.UnregisterExternalOperation(FMeshAddAssetUserData::StaticStruct());
}
