// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuR/External/Operation.h"

#include "MeshAddAssetUserData.generated.h"


USTRUCT(DisplayName = "Mesh Add Asset User Data")
struct FMeshAddAssetUserData : public UE::Mutable::FExternalOperation
{
	GENERATED_BODY()
	
	virtual ~FMeshAddAssetUserData() override = default;
	virtual TArray<TPair<FText, const UScriptStruct*>> GetInputs() const override;
	virtual TPair<FText, const UScriptStruct*> GetOutput() const override;
	virtual void Evaluate(UE::Mutable::FContext& Context) const override;
	
	static const FText TextInputMesh;
	
	static const FText TextInputAssetUserData;
};