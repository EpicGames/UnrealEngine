// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"

#include "MutableAssetUserData.generated.h"

class UAssetUserData;

USTRUCT(BlueprintType, meta = (DisplayName = "Asset User Data", PinColor = "#949494"))
struct FMutableAssetUserData
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Transient, Category = AssetUserData)
	TObjectPtr<UAssetUserData> AssetUserData;
};